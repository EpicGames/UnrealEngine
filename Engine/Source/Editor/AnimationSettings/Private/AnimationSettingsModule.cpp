// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationSettingsModule.h"

#include "SSettingsEditorCheckoutNotice.h"
#include "Animation/AnimationSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "StructUtils/UserDefinedStruct.h"


IMPLEMENT_MODULE(FAnimationSettingsModule, AnimationSettings);

DEFINE_LOG_CATEGORY(LogAnimationSettings);

#define LOCTEXT_NAMESPACE "AnimationSettings"

void FAnimationSettingsModule::OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldName)
{
	if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InAssetData.GetAsset()))
	{
		if (UAnimationSettings* Settings = UAnimationSettings::Get())
		{
			TArray<int32> Indices;
			for (int32 Index = 0; Index < Settings->UserDefinedStructAttributes.Num(); Index++)
			{
				const TSoftObjectPtr<UUserDefinedStruct>& StructPtr = Settings->UserDefinedStructAttributes[Index];
				UUserDefinedStruct* Struct = StructPtr.Get();
				if (Struct == UserDefinedStruct)
				{
					Indices.Add(Index);
				}
			}

			if (Indices.Num() > 0)
			{
				for (int32 Index : Indices)
				{
					Settings->UserDefinedStructAttributes[Index] = UserDefinedStruct;
				}


				FString RelativePath;
				bool bIsSourceControlled = false;
				bool bIsNewFile = false;

				check(Settings->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig));
				
				// Attempt to checkout the file automatically
				RelativePath = Settings->GetDefaultConfigFilename();
				bIsSourceControlled = true;
				FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

				if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullPath))
				{
					bIsNewFile = true;
				}

				if (!bIsSourceControlled || !SettingsHelpers::CheckOutOrAddFile(FullPath))
				{
					SettingsHelpers::MakeWritable(FullPath);
				}

				Settings->TryUpdateDefaultConfigFile();

				if (bIsNewFile && bIsSourceControlled)
				{
					SettingsHelpers::CheckOutOrAddFile(FullPath);
				}
			}	
		}
	}	
}

void FAnimationSettingsModule::StartupModule()
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
    
   OnAssetRenamedHandle = AssetRegistryModule.Get().OnAssetRenamed().AddStatic(&FAnimationSettingsModule::OnAssetRenamed);	
}

void FAnimationSettingsModule::ShutdownModule()
{
	if (const FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		if (IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet())
		{
			AssetRegistry->OnAssetRenamed().Remove(OnAssetRenamedHandle);
		}
	}
}



#undef LOCTEXT_NAMESPACE

