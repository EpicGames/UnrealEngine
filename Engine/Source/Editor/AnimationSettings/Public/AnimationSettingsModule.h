// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "AssetRegistry/AssetData.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAnimationSettings, Log, All);

class FAnimationSettingsModule : public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldName);

	FDelegateHandle OnAssetRenamedHandle;
};

